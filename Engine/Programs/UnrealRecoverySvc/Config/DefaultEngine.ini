[URL]
GameNameShort=UDRS

[Plugins]
+ProgramEnabledPlugins="UdpMessaging"
+ProgramEnabledPlugins="ConcertSyncServer"

[/Script/UdpMessaging.UdpMessagingSettings]
EnableTransport=True
MulticastTimeToLive=0
